{
	"FileVersion" : 3,
	"Version" : 2,
	"VersionName" : "2.0",
	"FriendlyName" : "Android Media Player",
	"Description" : "Implements a media player using the Android Media library.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "https://forums.unrealengine.com/showthread.php?46879-Media-Framework-Documentation-for-4-5-Preview",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "AndroidMedia",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PreLoadingScreen",
			"PlatformAllowList" : [ "Android" ]
		},
		{
			"Name" : "AndroidMediaEditor",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AndroidMediaFactory",
			"Type" : "Editor",
			"LoadingPhase" : "PostEngineInit"
		},
		{
			"Name" : "AndroidMediaFactory",
			"Type" : "RuntimeNoCommandlet",
			"LoadingPhase" : "PostEngineInit",
			"PlatformAllowList" : [ "Android" ]
		}
	]
}