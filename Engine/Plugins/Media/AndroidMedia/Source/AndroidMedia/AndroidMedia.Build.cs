// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AndroidMedia : ModuleRules
	{
		public AndroidMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"AndroidMediaFactory",
					"Core",
					"Engine",
					"MediaUtils",
					"RenderCore",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"Media",
				});
		}
	}
}
