// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidMediaPrivate.h"

#include "Modules/ModuleManager.h"

#include "IAndroidMediaModule.h"
#include "Player/AndroidMediaPlayer.h"


DEFINE_LOG_CATEGORY(LogAndroidMedia);


/**
 * Implements the AndroidMedia module.
 */
class FAndroidMediaModule
	: public IAndroidMediaModule
{
public:

	//~ IAndroidMediaModule interface

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		if (!IsSupported())
		{
			return nullptr;
		}

		return MakeShared<FAndroidMediaPlayer, ESPMode::ThreadSafe>(EventSink);
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

protected:

	/**
	 * Check whether media is supported on the running device.
	 *
	 * @return true if media is supported, false otherwise.
	 */
	bool IsSupported()
	{
		return (FAndroidMisc::GetAndroidBuildVersion() >= 14);
	}
};


IMPLEMENT_MODULE(FAndroidMediaModule, AndroidMedia)
