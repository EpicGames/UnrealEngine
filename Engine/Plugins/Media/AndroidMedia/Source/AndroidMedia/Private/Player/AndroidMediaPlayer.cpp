// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidMediaPlayer.h"
#include "AndroidMediaPrivate.h"

#include "Android/AndroidPlatformFile.h"
#include "AndroidMediaSettings.h"
#include "IMediaEventSink.h"
#include "MediaSamples.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "RHI.h"
#include "RHIResources.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"
#include "ExternalTexture.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"
#include "IPlatformFilePak.h"

#include "Android/AndroidJavaMediaPlayer.h"
#include "AndroidMediaTextureSample.h"

#define ANDROIDMEDIAPLAYER_USE_EXTERNALTEXTURE 1
#define ANDROIDMEDIAPLAYER_USE_PREPAREASYNC 1
#define ANDROIDMEDIAPLAYER_USE_NATIVELOGGING 1

#define LOCTEXT_NAMESPACE "FAndroidMediaPlayer"


/* FAndroidMediaPlayer structors
 *****************************************************************************/

FAndroidMediaPlayer::FAndroidMediaPlayer(IMediaEventSink& InEventSink)
	: CurrentState(EMediaState::Closed)
	, bLooping(false)
	, EventSink(InEventSink)
#if WITH_ENGINE
	, JavaMediaPlayer(MakeShared<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe>(false, FAndroidMisc::ShouldUseVulkan(), true))
#else
	, JavaMediaPlayer(MakeShared<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe>(true, FAndroidMisc::ShouldUseVulkan(), true))
#endif
	, Samples(MakeShared<FMediaSamples, ESPMode::ThreadSafe>())
	, SelectedAudioTrack(INDEX_NONE)
	, SelectedCaptionTrack(INDEX_NONE)
	, SelectedVideoTrack(INDEX_NONE)
	, VideoSamplePool(new FAndroidMediaTextureSamplePool)
	, bEnterBackground(false)
{
	check(JavaMediaPlayer.IsValid());
	check(Samples.IsValid());

	CurrentState = (JavaMediaPlayer.IsValid() && Samples.IsValid())? EMediaState::Closed : EMediaState::Error;
}


FAndroidMediaPlayer::~FAndroidMediaPlayer()
{
	Close();

	if (JavaMediaPlayer.IsValid())
	{
#if ANDROIDMEDIAPLAYER_USE_EXTERNALTEXTURE
		if (GSupportsImageExternal && !FAndroidMisc::ShouldUseVulkan())
		{
			// Unregister the external texture on render thread
			FTextureRHIRef VideoTexture = JavaMediaPlayer->GetVideoTexture();

			JavaMediaPlayer->SetVideoTexture(nullptr);
			JavaMediaPlayer->Reset();
			JavaMediaPlayer->Release();

			struct FReleaseVideoResourcesParams
			{
				FTextureRHIRef VideoTexture;
				FGuid PlayerGuid;
			};

			FReleaseVideoResourcesParams ReleaseVideoResourcesParams = { VideoTexture, PlayerGuid };

			ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
				[ReleaseVideoResourcesParams](FRHICommandListImmediate& RHICmdList)
				{
					#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("~FAndroidMediaPlayer: Unregister Guid: %s"), *ReleaseVideoResourcesParams.PlayerGuid.ToString());
					#endif

					FExternalTextureRegistry::Get().UnregisterExternalTexture(ReleaseVideoResourcesParams.PlayerGuid);

					// @todo: this causes a crash
//					Params.VideoTexture->Release();
				});
		}
		else
#endif // ANDROIDMEDIAPLAYER_USE_EXTERNALTEXTURE
		{
			JavaMediaPlayer->SetVideoTexture(nullptr);
			JavaMediaPlayer->Reset();
			JavaMediaPlayer->Release();
		}
	}

	delete VideoSamplePool;
	VideoSamplePool = nullptr;
}


/* IMediaPlayer interface
 *****************************************************************************/

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
// Useful for debugging
static void DumpState(EMediaState state)
{
	switch (state)
	{
	case EMediaState::Closed:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Closed"));
		break;
	case EMediaState::Error:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Error"));
		break;
	case EMediaState::Paused:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Paused"));
		break;
	case EMediaState::Playing:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Playing"));
		break;
	case EMediaState::Preparing:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Preparing"));
		break;
	case EMediaState::Stopped:
		FPlatformMisc::LowLevelOutputDebugString(TEXT("AndroidMediaPlayer: CurrentState = Stopped"));
		break;
	}
}
#endif

void FAndroidMediaPlayer::Close()
{
	#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Close() - %s"), *PlayerGuid.ToString());
	#endif

	if (CurrentState == EMediaState::Closed)
	{
		return;
	}

	CurrentState = EMediaState::Closed;

	// remove delegates if registered
	if (ResumeHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
		ResumeHandle.Reset();
	}
 
	if (PauseHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
		PauseHandle.Reset();
	}

	bLooping = false;

	if (JavaMediaPlayer.IsValid())
	{
		JavaMediaPlayer->SetLooping(false);
//		JavaMediaPlayer->SetVideoTextureValid(false);
		JavaMediaPlayer->Stop();
		JavaMediaPlayer->Reset();
	}

	VideoSamplePool->Reset();

	SelectedAudioTrack = INDEX_NONE;
	SelectedCaptionTrack = INDEX_NONE;
	SelectedVideoTrack = INDEX_NONE;

	AudioTracks.Empty();
	CaptionTracks.Empty();
	VideoTracks.Empty();

	Info.Empty();
	MediaUrl = FString();

	// notify listeners
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaClosed);
}


IMediaCache& FAndroidMediaPlayer::GetCache()
{
	return *this;
}


IMediaControls& FAndroidMediaPlayer::GetControls()
{
	return *this;
}


FString FAndroidMediaPlayer::GetInfo() const
{
	return Info;
}


FGuid FAndroidMediaPlayer::GetPlayerPluginGUID() const
{
	static FGuid PlayerPluginGUID(0x894a9ab3, 0xb44d4373, 0x87a7dd0c, 0x9cbd9613);
	return PlayerPluginGUID;
}


IMediaSamples& FAndroidMediaPlayer::GetSamples()
{
	return *Samples.Get();
}


FString FAndroidMediaPlayer::GetStats() const
{
	return TEXT("AndroidMedia stats information not implemented yet");
}


IMediaTracks& FAndroidMediaPlayer::GetTracks()
{
	return *this;
}


FString FAndroidMediaPlayer::GetUrl() const
{
	return MediaUrl;
}


IMediaView& FAndroidMediaPlayer::GetView()
{
	return *this;
}


bool FAndroidMediaPlayer::Open(const FString& Url, const IMediaOptions* /*Options*/)
{
	#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Open(%s) - %s"), *Url, *PlayerGuid.ToString());
	#endif

	if (CurrentState == EMediaState::Error)
	{
		return false;
	}

	Close();

	if ((Url.IsEmpty()))
	{
		return false;
	}

	MediaUrl = Url;

	// open the media
	if (Url.StartsWith(TEXT("file://")))
	{
		FString FilePath = Url.RightChop(7);
		FPaths::NormalizeFilename(FilePath);

		IAndroidPlatformFile& PlatformFile = IAndroidPlatformFile::GetPlatformPhysical();

		// make sure that file exists
		if (!PlatformFile.FileExists(*FilePath))
		{
			// possible it is in a PAK file
			FPakPlatformFile* PakPlatformFile = (FPakPlatformFile*)(FPlatformFileManager::Get().FindPlatformFile(FPakPlatformFile::GetTypeName()));

			TRefCountPtr<FPakFile> PakFile;
			FPakEntry FileEntry;
			if (PakPlatformFile == nullptr || !PakPlatformFile->FindFileInPakFiles(*FilePath, &PakFile, &FileEntry))
			{
				UE_LOGF(LogAndroidMedia, Warning, "File doesn't exist %ls.", *FilePath);
				return false;
			}

			// is it a simple case (can just use file datasource)?
			if (FileEntry.CompressionMethodIndex == 0 && !FileEntry.IsEncrypted())
			{
				FString PakFilename = PakFile->GetFilename();
				int64 PakHeaderSize = FileEntry.GetSerializedSize(PakFile->GetInfo().Version);
				int64 OffsetInPak = FileEntry.Offset + PakHeaderSize;
				int64 FileSize = FileEntry.Size;

				//UE_LOGF(LogAndroidMedia, Warning, "not compressed or encrypted PAK: Filename for PAK is: %ls", *PakFilename);
				//UE_LOGF(LogAndroidMedia, Warning, "PAK Offset: %lld,  Size: %lld (header=%lld)", OffsetInPak, FileSize, PakHeaderSize);

				int64 FileOffset = PlatformFile.FileStartOffset(*PakFilename) + OffsetInPak;
				FString FileRootPath = PlatformFile.FileRootPath(*PakFilename);

				//UE_LOGF(LogAndroidMedia, Warning, "Final Offset: %lld in %ls", FileOffset, *FileRootPath);

				if (!JavaMediaPlayer->SetDataSource(FileRootPath, FileOffset, FileSize))
				{
					UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for file in PAK %ls", *FilePath);
					return false;
				}
			}
			else
			{
				// only support media data source on Android 6.0+
				if (FAndroidMisc::GetAndroidBuildVersion() < 23)
				{
					UE_LOGF(LogAndroidMedia, Warning, "File cannot be played on this version of Android due to PAK file with compression or encryption: %ls.", *FilePath);
					return false;
				}

				TSharedRef<FArchive, ESPMode::ThreadSafe> Archive = MakeShareable(IFileManager::Get().CreateFileReader(*FilePath));
				if (!JavaMediaPlayer->SetDataSource(Archive))
				{
					UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for archive %ls", *FilePath);
					return false;
				}
			}
		}
		else
		{
			// get information about media
			int64 FileOffset = PlatformFile.FileStartOffset(*FilePath);
			int64 FileSize = PlatformFile.FileSize(*FilePath);
			FString FileRootPath = PlatformFile.FileRootPath(*FilePath);

			// play movie as a file or asset
			if (PlatformFile.IsAsset(*FilePath))
			{
				if (!JavaMediaPlayer->SetDataSource(PlatformFile.GetAssetManager(), FileRootPath, FileOffset, FileSize))
				{
					UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for asset %ls", *FilePath);
					return false;
				}
			}
			else
			{
				if (!JavaMediaPlayer->SetDataSource(FileRootPath, FileOffset, FileSize))
				{
					UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for file %ls", *FilePath);
					return false;
				}
			}
		}
	}
	else
	{
		// open remote media
		if (!JavaMediaPlayer->SetDataSource(Url))
		{
			UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for URL %ls", *Url);
			return false;
		}
	}

	// prepare media
	MediaUrl = Url;

#if ANDROIDMEDIAPLAYER_USE_PREPAREASYNC
	if (!JavaMediaPlayer->PrepareAsync())
	{
		UE_LOGF(LogAndroidMedia, Warning, "Failed to prepare media source %ls", *Url);
		return false;
	}

	CurrentState = EMediaState::Preparing;
	return true;
#else
	if (!JavaMediaPlayer->Prepare())
	{
		UE_LOGF(LogAndroidMedia, Warning, "Failed to prepare media source %ls", *Url);
		return false;
	}

	return InitializePlayer();
#endif
}


bool FAndroidMediaPlayer::Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* /*Options*/)
{
#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
	FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::OpenArchive(%s) - %s"), *OriginalUrl, *PlayerGuid.ToString());
#endif

	if (CurrentState == EMediaState::Error)
	{
		return false;
	}

	if (Archive->TotalSize() == 0)
	{
		UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Cannot open media from archive (archive is empty)", this);
		return false;
	}

	if (OriginalUrl.IsEmpty())
	{
		UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Cannot open media from archive (no original URL provided)", this);
		return false;
	}

	Close();

	// only support media data source on Android 6.0+
	if (FAndroidMisc::GetAndroidBuildVersion() < 23)
	{
		UE_LOGF(LogAndroidMedia, Warning, "File cannot be played on this version of Android due to PAK file with compression or encryption: %ls.", *OriginalUrl);
		return false;
	}

	if (!JavaMediaPlayer->SetDataSource(Archive))
	{
		UE_LOGF(LogAndroidMedia, Warning, "Failed to set data source for archive %ls", *OriginalUrl);
		return false;
	}

	// prepare media
	MediaUrl = OriginalUrl;

#if ANDROIDMEDIAPLAYER_USE_PREPAREASYNC
	if (!JavaMediaPlayer->PrepareAsync())
	{
		UE_LOGF(LogAndroidMedia, Warning, "Failed to prepare media source %ls", *OriginalUrl);
		return false;
	}

	CurrentState = EMediaState::Preparing;
	return true;
#else
	if (!JavaMediaPlayer->Prepare())
	{
		UE_LOGF(LogAndroidMedia, Warning, "Failed to prepare media source %ls", *OriginalUrl);
		return false;
	}

	return InitializePlayer();
#endif
}


void FAndroidMediaPlayer::SetGuid(const FGuid& Guid)
{
	PlayerGuid = Guid;

	#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("IMediaPlayer SetGuid: %s"), *PlayerGuid.ToString());
	#endif
}

// ==============================================================
// Used by TickFetch

static void ConditionalInitializeVideoTexture_RenderThread(FRHICommandListBase& RHICmdList, TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, FGuid PlayerGuid)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid())
	{
		return;
	}

	FTextureRHIRef VideoTexture = PinnedJavaMediaPlayer->GetVideoTexture();
	if (VideoTexture == nullptr)
	{
		const FRHITextureCreateDesc Desc =
			FRHITextureCreateDesc::Create2D(TEXT("VideoTexture"), 1, 1, PF_R8G8B8A8)
			.SetFlags(ETextureCreateFlags::External)
			.SetInitialState(ERHIAccess::SRVMask);

		VideoTexture = RHICmdList.CreateTexture(Desc);
		PinnedJavaMediaPlayer->SetVideoTexture(VideoTexture);

		if (VideoTexture == nullptr)
		{
			UE_LOGF(LogAndroidMedia, Warning, "CreateTexture failed!");
			return;
		}

		PinnedJavaMediaPlayer->SetVideoTextureValid(false);

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Created VideoTexture for Player: %s"), *PlayerGuid.ToString());
#endif
	}
}

static void DoUpdateExternalMediaSampleExecute(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, FGuid PlayerGuid)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid())
	{
		return;
	}

	FTextureRHIRef VideoTexture = PinnedJavaMediaPlayer->GetVideoTexture();
	if (VideoTexture == nullptr)
	{
		return;
	}

	int32 TextureId = *reinterpret_cast<int32*>(VideoTexture->GetNativeResource());
	int32 CurrentFramePosition = 0;
	bool bRegionChanged = false;
	if (PinnedJavaMediaPlayer->UpdateVideoFrame(TextureId, &CurrentFramePosition, &bRegionChanged))
	{
#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		//			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: %d - %s"), CurrentFramePosition, *Params.PlayerGuid.ToString());
#endif

		// if region changed, need to reregister UV scale/offset
		if (bRegionChanged)
		{
			PinnedJavaMediaPlayer->SetVideoTextureValid(false);

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: New UV Scale/Offset = {%f, %f}, {%f, %f} - %s"),
				PinnedJavaMediaPlayer->GetUScale(), PinnedJavaMediaPlayer->GetUOffset(),
				PinnedJavaMediaPlayer->GetVScale(), PinnedJavaMediaPlayer->GetVOffset(), *PlayerGuid.ToString());
#endif
		}
	}
}

struct FRHICommandUpdateExternalMediaSample final : public FRHICommand<FRHICommandUpdateExternalMediaSample>
{
	TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
	FGuid PlayerGuid;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateExternalMediaSample(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> InJavaMediaPlayerPtr, FGuid InPlayerGuid)
		: JavaMediaPlayerPtr(InJavaMediaPlayerPtr)
		, PlayerGuid(InPlayerGuid)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateExternalMediaSample_Execute);
		DoUpdateExternalMediaSampleExecute(JavaMediaPlayerPtr, PlayerGuid);
	}
};

static void ConditionalInitializeVideoSample_RenderThread(FRHICommandListBase& RHICmdList, TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample)
{
	if (VideoSample->GetTexture() == nullptr || VideoSample->GetTexture()->GetSizeXY() != VideoSample->GetOutputDim())
	{
		VideoSample->InitializeTexture(RHICmdList, FTimespan::Zero());
	}
}

static void DoUpdateTextureMediaSampleExecute(FRHICommandListBase& RHICmdList, TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	const int32 CurrentFramePosition = PinnedJavaMediaPlayer->GetCurrentPosition();
	const FTimespan Time = FTimespan::FromMilliseconds(CurrentFramePosition);

	// write frame into texture
	FRHITexture* Texture = VideoSample->InitializeTexture(RHICmdList, Time);

	if (Texture != nullptr)
	{
		int32 Resource = *reinterpret_cast<int32*>(Texture->GetNativeResource());
		if (!PinnedJavaMediaPlayer->GetVideoLastFrame(Resource))
		{
			return;
		}
	}

	PinnedSamples->AddVideo(VideoSample);
}

struct FRHICommandUpdateTextureMediaSample final : public FRHICommand<FRHICommandUpdateTextureMediaSample>
{
	TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateTextureMediaSample(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> InJavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> InVideoSample)
		: JavaMediaPlayerPtr(InJavaMediaPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
	{
	}
	void Execute(FRHICommandListBase& RHICmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateTextureMediaSample_Execute);
		DoUpdateTextureMediaSampleExecute(RHICmdList, JavaMediaPlayerPtr, SamplesPtr, VideoSample);
	}
};

static void DoUpdateBufferMediaSampleExecute(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample, int32 SampleCount)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	const int32 CurrentFramePosition = PinnedJavaMediaPlayer->GetCurrentPosition();
	const FTimespan Time = FTimespan::FromMilliseconds(CurrentFramePosition);

	// write frame into buffer
	void* Buffer = nullptr;
	int64 BufferCount = 0;

	if (!PinnedJavaMediaPlayer->GetVideoLastFrameData(Buffer, BufferCount))
	{
		return;
	}

	if (BufferCount != SampleCount)
	{
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMediaPlayer::Fetch: Sample count mismatch (Buffer=%d, Available=%d"), BufferCount, SampleCount);
	}
	check(SampleCount <= BufferCount);

	// must make a copy (buffer is owned by Java, not us!)
	VideoSample->InitializeBuffer(Buffer, Time, true);

	PinnedSamples->AddVideo(VideoSample);
}

struct FRHICommandUpdateBufferMediaSample final : public FRHICommand<FRHICommandUpdateBufferMediaSample>
{
	TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
	int32 SampleCount;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateBufferMediaSample(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> InJavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> InVideoSample, int32 InSampleCount)
		: JavaMediaPlayerPtr(InJavaMediaPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
		, SampleCount(InSampleCount)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateBufferMediaSample_Execute);
		DoUpdateBufferMediaSampleExecute(JavaMediaPlayerPtr, SamplesPtr, VideoSample, SampleCount);
	}
};

static void DoUpdateMediaFrameDataMediaSampleExecute(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr,
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample)
{
	auto PinnedJavaMediaPlayer = JavaMediaPlayerPtr.Pin();
	auto PinnedSamples = SamplesPtr.Pin();

	if (!PinnedJavaMediaPlayer.IsValid() || !PinnedSamples.IsValid())
	{
		return;
	}

	const int32 CurrentFramePosition = PinnedJavaMediaPlayer->GetCurrentPosition();
	const FTimespan Time = FTimespan::FromMilliseconds(CurrentFramePosition);

	jobject FrameDataGlobalRef = PinnedJavaMediaPlayer->GetVideoLastMediaFrameData();
	if (FrameDataGlobalRef == nullptr)
	{
		return;
	}

	VideoSample->InitializeMediaFrameData(FrameDataGlobalRef, Time);

	PinnedSamples->AddVideo(VideoSample);
}

struct FRHICommandUpdateMediaFrameDataMediaSample final : public FRHICommand<FRHICommandUpdateMediaFrameDataMediaSample>
{
	TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
	TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
	TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;

	FORCEINLINE_DEBUGGABLE FRHICommandUpdateMediaFrameDataMediaSample(TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> InJavaMediaPlayerPtr, TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> InSamplesPtr,
		TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> InVideoSample)
		: JavaMediaPlayerPtr(InJavaMediaPlayerPtr)
		, SamplesPtr(InSamplesPtr)
		, VideoSample(InVideoSample)
	{
	}
	void Execute(FRHICommandListBase& CmdList)
	{
		QUICK_SCOPE_CYCLE_COUNTER(STAT_FRHICommandUpdateTextureMediaSample_Execute);
		DoUpdateMediaFrameDataMediaSampleExecute(JavaMediaPlayerPtr, SamplesPtr, VideoSample);
	}
};

// ==============================================================

void FAndroidMediaPlayer::TickFetch(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	if (CurrentState != EMediaState::Playing && CurrentState != EMediaState::Paused)
	{
		return;
	}

	if (!JavaMediaPlayer.IsValid() || !VideoTracks.IsValidIndex(SelectedVideoTrack))
	{
		return;
	}

	// deal with resolution changes (usually from streams)
	if (JavaMediaPlayer->DidResolutionChange())
	{
		JavaMediaPlayer->SetVideoTextureValid(false);

		// The video track dimensions need updating
		FIntPoint Dimensions = FIntPoint(JavaMediaPlayer->GetVideoWidth(), JavaMediaPlayer->GetVideoHeight());
		VideoTracks[SelectedVideoTrack].Dimensions = Dimensions;
	}

#if WITH_ENGINE

	if (FAndroidMisc::ShouldUseVulkan())
	{
		// create new video sample
		const FJavaAndroidMediaPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		auto VideoSample = VideoSamplePool->AcquireShared();

		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}

		// populate & add sample (on render thread)
		//const auto Settings = GetDefault<UAndroidMediaSettings>();

		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
			TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
			int32 SampleCount;
		}
		Params = { JavaMediaPlayer, Samples, VideoSample, (int32)(VideoTrack.Dimensions.X * VideoTrack.Dimensions.Y * sizeof(int32)) };

		if (!JavaMediaPlayer->IsUsingBitmapRendererImageReader())
		{
			ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					DoUpdateBufferMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleCount);
				});
		}
		else
		{
			ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					DoUpdateMediaFrameDataMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample);
				});
		}
	}
#if ANDROIDMEDIAPLAYER_USE_EXTERNALTEXTURE
	else if (GSupportsImageExternal)
	{
		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
			FGuid PlayerGuid;
			TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;
		};

		auto VideoSample = VideoSamplePool->AcquireShared();
		const FJavaAndroidMediaPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}
			
		FWriteVideoSampleParams Params = { JavaMediaPlayer, PlayerGuid, VideoSample, Samples };

		if (!JavaMediaPlayer->IsUsingBitmapRendererImageReader())
		{
			ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					ConditionalInitializeVideoTexture_RenderThread(RHICmdList, Params.JavaMediaPlayerPtr, Params.PlayerGuid);
					if (IsRunningRHIInSeparateThread())
					{
						new (RHICmdList.AllocCommand<FRHICommandUpdateExternalMediaSample>()) FRHICommandUpdateExternalMediaSample(Params.JavaMediaPlayerPtr, Params.PlayerGuid);
						// wait for DoUpdateExternalMediaSampleExecute to complete.
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
					}
					else
					{
						DoUpdateExternalMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.PlayerGuid);
					}

					auto PinnedJavaMediaPlayer = Params.JavaMediaPlayerPtr.Pin();
					auto PinnedSamples = Params.Samples.Pin();

					if (!PinnedJavaMediaPlayer.IsValid() || !PinnedSamples.IsValid())
					{
						return;
					}

					FTextureRHIRef VideoTexture = PinnedJavaMediaPlayer->GetVideoTexture();
					if (VideoTexture != nullptr && !PinnedJavaMediaPlayer->IsVideoTextureValid())
					{
#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
						FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Fetch RT: Register Guid: %s"), *Params.PlayerGuid.ToString());
#endif
						FSamplerStateInitializerRHI SamplerStateInitializer(SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp);
						FSamplerStateRHIRef SamplerStateRHI = RHICreateSamplerState(SamplerStateInitializer);
						FExternalTextureRegistry::Get().RegisterExternalTexture(Params.PlayerGuid, VideoTexture, SamplerStateRHI, FLinearColor(PinnedJavaMediaPlayer->GetUScale(), 0.0f, 0.0f, PinnedJavaMediaPlayer->GetVScale()), FLinearColor(PinnedJavaMediaPlayer->GetUOffset(), PinnedJavaMediaPlayer->GetVOffset(), 0.0f, 0.0f));

						PinnedJavaMediaPlayer->SetVideoTextureValid(true);
					}

					const int32 CurrentFramePosition = PinnedJavaMediaPlayer->GetCurrentPosition();
					const FTimespan Time = FTimespan::FromMilliseconds(CurrentFramePosition);

					Params.VideoSample->InitializeExternalTexture(Time);
					// Add a dummy video sample
					// (actual data flows "on the side" via external texture mechanism)
					PinnedSamples->AddVideo(Params.VideoSample);
				});
		}
		else
		{
			ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
				[Params](FRHICommandListImmediate& RHICmdList)
				{
					if (IsRunningRHIInSeparateThread())
					{
						new (RHICmdList.AllocCommand<FRHICommandUpdateMediaFrameDataMediaSample>()) FRHICommandUpdateMediaFrameDataMediaSample(Params.JavaMediaPlayerPtr, Params.Samples, Params.VideoSample);
						// wait for DoUpdateExternalMediaSampleExecute to complete.
						RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThread);
					}
					else
					{
						DoUpdateMediaFrameDataMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.Samples, Params.VideoSample);
					}
				});
		}
	}
#endif // ANDROIDMEDIAPLAYER_USE_EXTERNALTEXTURE
	else
	{
		// create new video sample
		const FJavaAndroidMediaPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
		auto VideoSample = VideoSamplePool->AcquireShared();

		if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
		{
			return;
		}

		// populate & add sample (on render thread)
		//const auto Settings = GetDefault<UAndroidMediaSettings>();

		struct FWriteVideoSampleParams
		{
			TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
			TWeakPtr<FMediaSamples, ESPMode::ThreadSafe> SamplesPtr;
			TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
		}
		Params = { JavaMediaPlayer, Samples, VideoSample };

		ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				ConditionalInitializeVideoSample_RenderThread(RHICmdList, Params.VideoSample);
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateTextureMediaSample>()) FRHICommandUpdateTextureMediaSample(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample);
				}
				else
				{
					DoUpdateTextureMediaSampleExecute(RHICmdList, Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample);
				}
			});
	}

#else

	// create new video sample
	const FJavaAndroidMediaPlayer::FVideoTrack VideoTrack = VideoTracks[SelectedVideoTrack];
	auto VideoSample = VideoSamplePool->AcquireShared();

	if (!VideoSample->Initialize(VideoTrack.Dimensions, FTimespan::FromSeconds(1.0 / VideoTrack.FrameRate)))
	{
		return;
	}

	// populate & add sample (on render thread)
	//const auto Settings = GetDefault<UAndroidMediaSettings>();

	struct FWriteVideoSampleParams
	{
		TWeakPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayerPtr;
		TWeakPtr<FMediaSampleQueue, ESPMode::ThreadSafe> SamplesPtr;
		TSharedRef<FAndroidMediaTextureSample, ESPMode::ThreadSafe> VideoSample;
		int32 SampleCount;
	};

	FWriteVideoSampleParams Params = { JavaMediaPlayer, Samples, VideoSample, (int32)(VideoTrack.Dimensions.X * VideoTrack.Dimensions.Y * sizeof(int32)) };

	if (!JavaMediaPlayer->IsUsingBitmapRendererImageReader())
	{
		ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateBufferMediaSample>()) FRHICommandUpdateBufferMediaSample(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleCount);
				}
				else
				{
					DoUpdateBufferMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample, Params.SampleCount);
				}
			});
	}
	else
	{
		ENQUEUE_RENDER_COMMAND(AndroidMediaPlayerWriteVideoSample)(
			[Params](FRHICommandListImmediate& RHICmdList)
			{
				if (IsRunningRHIInSeparateThread())
				{
					new (RHICmdList.AllocCommand<FRHICommandUpdateMediaFrameDataMediaSample>()) FRHICommandUpdateMediaFrameDataMediaSample(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample);
				}
				else
				{
					DoUpdateMediaFrameDataMediaSampleExecute(Params.JavaMediaPlayerPtr, Params.SamplesPtr, Params.VideoSample);
				}
			});
	}
#endif //WITH_ENGINE
}


void FAndroidMediaPlayer::TickInput(FTimespan DeltaTime, FTimespan /*Timecode*/)
{
	if (CurrentState != EMediaState::Playing)
	{
		// remove delegates if registered
		if (ResumeHandle.IsValid())
		{
			FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
			ResumeHandle.Reset();
		}

		if (PauseHandle.IsValid())
		{
			FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
			PauseHandle.Reset();
		}

		if (!JavaMediaPlayer.IsValid())
		{
			return;
		}

#if ANDROIDMEDIAPLAYER_USE_PREPAREASYNC
		// if preparing, see if finished
		if (CurrentState == EMediaState::Preparing)
		{
			if (!JavaMediaPlayer.IsValid())
			{
				return;
			}

			if (JavaMediaPlayer->IsPrepared())
			{
				InitializePlayer();
			}
		}
		else
#endif
		if (CurrentState == EMediaState::Stopped)
		{
			if (JavaMediaPlayer->DidComplete())
			{
				EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Tick - PlaybackEndReached - stopped - %s"), *PlayerGuid.ToString());
#endif
			}
		}

		return;
	}

	if (!JavaMediaPlayer.IsValid())
	{
		return;
	}

	// register delegate if not registered
	if (!ResumeHandle.IsValid())
	{
		ResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FAndroidMediaPlayer::HandleApplicationHasEnteredForeground);
	}
	if (!PauseHandle.IsValid())
	{
		PauseHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FAndroidMediaPlayer::HandleApplicationWillEnterBackground);
	}

	// generate events
	if (!JavaMediaPlayer->IsPlaying() && !bEnterBackground)
	{
		// might catch it restarting the loop so ignore if looping
		if (!bLooping)
		{
			CurrentState = EMediaState::Stopped;
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Tick - PlaybackSuspended - !playing - %s"), *PlayerGuid.ToString());
#endif
		}

		if (JavaMediaPlayer->DidComplete())
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

			#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
				FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Tick - PlaybackEndReached - !playing - %s"), *PlayerGuid.ToString());
			#endif
		}
	}
	else if (JavaMediaPlayer->DidComplete())
	{
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

		#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Tick - PlaybackEndReached - DidComplete true - %s"), *PlayerGuid.ToString());
		#endif
	}
}


/* FAndroidMediaPlayer implementation
 *****************************************************************************/

bool FAndroidMediaPlayer::InitializePlayer()
{
	#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
		FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::InitializePlayer %s"), *PlayerGuid.ToString());
	#endif

	if (ResumeHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(ResumeHandle);
		ResumeHandle.Reset();
	}

	if (PauseHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(PauseHandle);
		PauseHandle.Reset();
	}

	JavaMediaPlayer->GetAudioTracks(AudioTracks);
	JavaMediaPlayer->GetCaptionTracks(CaptionTracks);
	JavaMediaPlayer->GetVideoTracks(VideoTracks);

	for (FJavaAndroidMediaPlayer::FVideoTrack Track : VideoTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Video\n");
//		Info += FString::Printf(TEXT("    Duration: %s\n"), *FTimespan::FromMilliseconds(Track->StreamInfo.duration).ToString());
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += FString::Printf(TEXT("    FrameRate: %0.1f\n"), Track.FrameRate);
		Info += FString::Printf(TEXT("    Dimensions: %i x %i\n"), Track.Dimensions.X, Track.Dimensions.Y);
		Info += TEXT("\n");
	}

	for (FJavaAndroidMediaPlayer::FAudioTrack Track : AudioTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Audio\n");
//		Info += FString::Printf(TEXT("    Duration: %s\n"), *FTimespan::FromMilliseconds(Track->StreamInfo.duration).ToString());
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += FString::Printf(TEXT("    Channels: %i\n"), Track.Channels);
		Info += FString::Printf(TEXT("    Sample Rate: %i Hz\n"), Track.SampleRate);
		Info += TEXT("\n");
	}

	for (FJavaAndroidMediaPlayer::FCaptionTrack Track : CaptionTracks)
	{
		Info += FString::Printf(TEXT("Stream %i\n"), Track.Index);
		Info += TEXT("    Type: Caption\n");
		Info += FString::Printf(TEXT("    MimeType: %s\n"), *Track.MimeType);
		Info += FString::Printf(TEXT("    Language: %s\n"), *Track.Language);
		Info += TEXT("\n");
	}

	// Select first audio and video track by default (have to force update first time by setting to none)
	if (AudioTracks.Num() > 0)
	{
		JavaMediaPlayer->SetAudioEnabled(true);
		SelectedAudioTrack = 0;
	}
	else
	{
		JavaMediaPlayer->SetAudioEnabled(false);
		SelectedAudioTrack = INDEX_NONE;
	}

	if (VideoTracks.Num() > 0)
	{
		JavaMediaPlayer->SetVideoEnabled(true);
		SelectedVideoTrack = 0;
	}
	else
	{
		JavaMediaPlayer->SetVideoEnabled(false);
		SelectedVideoTrack = INDEX_NONE;
	}

	CurrentState = EMediaState::Stopped;

	// notify listeners
	EventSink.ReceiveMediaEvent(EMediaEvent::TracksChanged);
	EventSink.ReceiveMediaEvent(EMediaEvent::MediaOpened);

	return true;
}


/* IMediaTracks interface
 *****************************************************************************/

bool FAndroidMediaPlayer::GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const
{
	if ((FormatIndex != 0) || !AudioTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	const FJavaAndroidMediaPlayer::FAudioTrack& Track = AudioTracks[TrackIndex];

	OutFormat.BitsPerSample = 16;
	OutFormat.NumChannels = Track.Channels;
	OutFormat.SampleRate = Track.SampleRate;
	OutFormat.TypeName = TEXT("Native");

	return true;
}


int32 FAndroidMediaPlayer::GetNumTracks(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.Num();

	case EMediaTrackType::Caption:
		return CaptionTracks.Num();

	case EMediaTrackType::Video:
		return VideoTracks.Num();

	default:
		return 0;
	}
}


int32 FAndroidMediaPlayer::GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const
{
	// @todo chrisb: add support for track formats
	return ((TrackIndex >= 0) && (TrackIndex < GetNumTracks(TrackType))) ? 1 : 0;
}


int32 FAndroidMediaPlayer::GetSelectedTrack(EMediaTrackType TrackType) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return SelectedAudioTrack;

	case EMediaTrackType::Caption:
		return SelectedCaptionTrack;

	case EMediaTrackType::Video:
		return SelectedVideoTrack;

	default:
		return INDEX_NONE;
	}
}


FText FAndroidMediaPlayer::GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(AudioTracks[TrackIndex].DisplayName);
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(CaptionTracks[TrackIndex].DisplayName);
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return FText::FromString(VideoTracks[TrackIndex].DisplayName);
		}

	default:
		break;
	}

	return FText::GetEmpty();
}


int32 FAndroidMediaPlayer::GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const
{
	// @todo chrisb: add support for track formats
	return (GetSelectedTrack(TrackType) != INDEX_NONE) ? 0 : INDEX_NONE;
}


FString FAndroidMediaPlayer::GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Language;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Language;
		}

	default:
		break;
	}

	return FString();
}


FString FAndroidMediaPlayer::GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (AudioTracks.IsValidIndex(TrackIndex))
		{
			return AudioTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Caption:
		if (CaptionTracks.IsValidIndex(TrackIndex))
		{
			return CaptionTracks[TrackIndex].Name;
		}
		break;

	case EMediaTrackType::Video:
		if (VideoTracks.IsValidIndex(TrackIndex))
		{
			return VideoTracks[TrackIndex].Name;
		}

	default:
		break;
	}

	return FString();
}


bool FAndroidMediaPlayer::GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const
{
	if ((FormatIndex != 0) || !VideoTracks.IsValidIndex(TrackIndex))
	{
		return false;
	}

	const FJavaAndroidMediaPlayer::FVideoTrack& Track = VideoTracks[TrackIndex];

	OutFormat.Dim = Track.Dimensions;
	OutFormat.FrameRate = Track.FrameRate;
	OutFormat.FrameRates = TRange<float>(Track.FrameRate);
	OutFormat.TypeName = TEXT("BGRA");

	return true;
}


bool FAndroidMediaPlayer::SelectTrack(EMediaTrackType TrackType, int32 TrackIndex)
{
	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		if (TrackIndex != SelectedAudioTrack)
		{
			UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Selecting audio track %i instead of %i (%i tracks)", this, TrackIndex, SelectedVideoTrack, AudioTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Disabling audio", this);

				JavaMediaPlayer->SetAudioEnabled(false);
			}
			else
			{
				if (!AudioTracks.IsValidIndex(TrackIndex) || !JavaMediaPlayer->SelectTrack(AudioTracks[TrackIndex].Index))
				{
					return false;
				}

				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Enabling audio", this);

				JavaMediaPlayer->SetAudioEnabled(true);
			}

			SelectedAudioTrack = TrackIndex;
		}
		break;

	case EMediaTrackType::Caption:
		if (TrackIndex != SelectedCaptionTrack)
		{
			UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Selecting caption track %i instead of %i (%i tracks)", this, TrackIndex, SelectedCaptionTrack, CaptionTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Disabling captions", this);
			}
			else
			{
				if (!CaptionTracks.IsValidIndex(TrackIndex) || JavaMediaPlayer->SelectTrack(CaptionTracks[TrackIndex].Index))
				{
					return false;
				}

				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Enabling captions", this);
			}

			SelectedCaptionTrack = TrackIndex;
		}
		break;

	case EMediaTrackType::Video:
		if (TrackIndex != SelectedVideoTrack)
		{
			UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Selecting video track %i instead of %i (%i tracks).", this, TrackIndex, SelectedVideoTrack, VideoTracks.Num());

			if (TrackIndex == INDEX_NONE)
			{
				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Disabling video", this);
				JavaMediaPlayer->SetVideoEnabled(false);
			}
			else
			{
				if (!VideoTracks.IsValidIndex(TrackIndex) || !JavaMediaPlayer->SelectTrack(VideoTracks[TrackIndex].Index))
				{
					return false;
				}

				UE_LOGF(LogAndroidMedia, VeryVerbose, "Player %p: Enabling video", this);
				JavaMediaPlayer->SetVideoEnabled(true);
			}

			SelectedVideoTrack = TrackIndex;
		}
		break;

	default:
		return false;
	}

	return true;
}


bool FAndroidMediaPlayer::SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex)
{
	if (FormatIndex != 0)
	{
		return false;
	}

	switch (TrackType)
	{
	case EMediaTrackType::Audio:
		return AudioTracks.IsValidIndex(TrackIndex);

	case EMediaTrackType::Caption:
		return CaptionTracks.IsValidIndex(TrackIndex);

	case EMediaTrackType::Video:
		return VideoTracks.IsValidIndex(TrackIndex);

	default:
		return false;
	}
}


/* IMediaControls interface
 *****************************************************************************/

bool FAndroidMediaPlayer::CanControl(EMediaControl Control) const
{
	if (Control == EMediaControl::Pause)
	{
		return (CurrentState == EMediaState::Playing);
	}

	if (Control == EMediaControl::Resume)
	{
		return ((CurrentState == EMediaState::Playing) || (CurrentState == EMediaState::Stopped));
	}

	if (Control == EMediaControl::Seek)
	{
		return ((CurrentState != EMediaState::Closed) && (CurrentState != EMediaState::Error));
	}

	return false;
}


FTimespan FAndroidMediaPlayer::GetDuration() const
{
	if (CurrentState == EMediaState::Error)
	{
		return FTimespan::Zero();
	}

	return FTimespan::FromMilliseconds(JavaMediaPlayer->GetDuration());
}


float FAndroidMediaPlayer::GetRate() const
{
	return (CurrentState == EMediaState::Playing) ? 1.0f : 0.0f;
}


EMediaState FAndroidMediaPlayer::GetState() const
{
	return CurrentState;
}


EMediaStatus FAndroidMediaPlayer::GetStatus() const
{
	return EMediaStatus::None;
}


TRangeSet<float> FAndroidMediaPlayer::GetSupportedRates(EMediaRateThinning /*Thinning*/) const
{
	TRangeSet<float> Result;

	Result.Add(TRange<float>(0.0f));
	Result.Add(TRange<float>(1.0f));

	return Result;
}


FTimespan FAndroidMediaPlayer::GetTime() const
{
	if ((CurrentState == EMediaState::Closed) || (CurrentState == EMediaState::Error))
	{
		return FTimespan::Zero();
	}

	return FTimespan::FromMilliseconds(JavaMediaPlayer->GetCurrentPosition());
}


bool FAndroidMediaPlayer::IsLooping() const
{
	return bLooping;
}


bool FAndroidMediaPlayer::Seek(const FTimespan& Time)
{
	UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Seeking to %ls", this, *Time.ToString());

	if ((CurrentState == EMediaState::Closed) ||
		(CurrentState == EMediaState::Error) ||
		(CurrentState == EMediaState::Preparing))
	{
		UE_LOGF(LogAndroidMedia, Warning, "Cannot seek while closed, preparing, or in error state");
		return false;
	}

	UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Seeking to %ls", this, *Time.ToString());

	JavaMediaPlayer->SeekTo(static_cast<int32>(Time.GetTotalMilliseconds()));
	EventSink.ReceiveMediaEvent(EMediaEvent::SeekCompleted);

	return true;
}


bool FAndroidMediaPlayer::SetLooping(bool Looping)
{
	bLooping = Looping;

	if (!JavaMediaPlayer.IsValid())
	{
		return false;
	}

	JavaMediaPlayer->SetLooping(Looping);

	return true;
}


bool FAndroidMediaPlayer::SetRate(float Rate)
{
	if ((CurrentState == EMediaState::Closed) ||
		(CurrentState == EMediaState::Error) ||
		(CurrentState == EMediaState::Preparing))
	{
		UE_LOGF(LogAndroidMedia, Warning, "Cannot set rate while closed, preparing, or in error state");
		return false;
	}

	if (Rate == GetRate())
	{
		return true; // rate already set
	}

	UE_LOGF(LogAndroidMedia, Verbose, "Player %p: Setting rate from to %f to %f", this, GetRate(), Rate);

	if (Rate == 0.0f)
	{
		JavaMediaPlayer->Pause();
		CurrentState = EMediaState::Paused;
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackSuspended);
	}
	else if (Rate == 1.0f)
	{
		JavaMediaPlayer->Start();
		CurrentState = EMediaState::Playing;
		EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackResumed);
	}
	else
	{
		UE_LOGF(LogAndroidMedia, Warning, "The rate %f is not supported", Rate);
		return false;
	}

	return true;
}


bool FAndroidMediaPlayer::SetNativeVolume(float Volume)
{
	if (JavaMediaPlayer.IsValid())
	{
		Volume = Volume < 0.0f ? 0.0f : (Volume < 1.0f ? Volume : 1.0f);
		JavaMediaPlayer->SetAudioVolume(Volume);
		return true;
	}
	return false;
}


/* FAndroidMediaPlayer callbacks
 *****************************************************************************/

void FAndroidMediaPlayer::HandleApplicationWillEnterBackground()
{
	// check state in case changed before ticked
	if ((CurrentState == EMediaState::Playing) && JavaMediaPlayer.IsValid())
	{
		bool bDidComplete = JavaMediaPlayer->DidComplete();
		
		if (!bDidComplete || bLooping)
		{
			JavaMediaPlayer->Pause();
		}
		else
		{
			CurrentState = EMediaState::Stopped;
		}

		if (bDidComplete)
		{
			EventSink.ReceiveMediaEvent(EMediaEvent::PlaybackEndReached);

#if ANDROIDMEDIAPLAYER_USE_NATIVELOGGING
			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("FAndroidMedia::Tick - PlaybackEndReached - stopped - %s"), *PlayerGuid.ToString());
#endif
		}
	}
	bEnterBackground = true;
}


void FAndroidMediaPlayer::HandleApplicationHasEnteredForeground()
{
	// check state in case changed before ticked
	if ((CurrentState == EMediaState::Playing) && JavaMediaPlayer.IsValid())
	{
		JavaMediaPlayer->Start();
	}
	bEnterBackground = false;
}


#undef LOCTEXT_NAMESPACE
