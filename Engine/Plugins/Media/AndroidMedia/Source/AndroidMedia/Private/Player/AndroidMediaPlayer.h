// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "Misc/Guid.h"
#include "IMediaCache.h"
#include "IMediaControls.h"
#include "IMediaPlayer.h"
#include "IMediaTracks.h"
#include "IMediaView.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

#include "Android/AndroidJavaMediaPlayer.h"

class FAndroidMediaTextureSamplePool;
class FMediaSamples;
class IMediaEventSink;


/*
 * Implements media playback using the Android MediaPlayer interface.
 */
class FAndroidMediaPlayer
	: public IMediaPlayer
	, protected IMediaCache
	, protected IMediaControls
	, protected IMediaTracks
	, protected IMediaView
{
public:
	
	/**
	 * Create and initialize a new instance.
	 *
	 * @param InEventSink The object that receives media events from this player.
	 */
	FAndroidMediaPlayer(IMediaEventSink& InEventSink);

	/** Virtual destructor. */
	virtual ~FAndroidMediaPlayer();

public:

	//~ IMediaPlayer interface

	virtual void Close() override;
	virtual IMediaCache& GetCache() override;
	virtual IMediaControls& GetControls() override;
	virtual FString GetInfo() const override;
	virtual FGuid GetPlayerPluginGUID() const override;
	virtual IMediaSamples& GetSamples() override;
	virtual FString GetStats() const override;
	virtual IMediaTracks& GetTracks() override;
	virtual FString GetUrl() const override;
	virtual IMediaView& GetView() override;
	virtual bool Open(const FString& Url, const IMediaOptions* Options) override;
	virtual bool Open(const TSharedRef<FArchive, ESPMode::ThreadSafe>& Archive, const FString& OriginalUrl, const IMediaOptions* Options) override;
	virtual void SetGuid(const FGuid& Guid) override;
	virtual void TickFetch(FTimespan DeltaTime, FTimespan Timecode) override;
	virtual void TickInput(FTimespan DeltaTime, FTimespan Timecode) override;

protected:

	/**
	 * Initialize the media player.
	 *
	 * @return true on success, false otherwise.
	 */
	bool InitializePlayer();

protected:

	//~ IMediaControls interface

	virtual bool CanControl(EMediaControl Control) const override;
	virtual FTimespan GetDuration() const override;
	virtual float GetRate() const override;
	virtual EMediaState GetState() const override;
	virtual EMediaStatus GetStatus() const override;
	virtual TRangeSet<float> GetSupportedRates(EMediaRateThinning Thinning) const override;
	virtual FTimespan GetTime() const override;
	virtual bool IsLooping() const override;
	virtual bool Seek(const FTimespan& Time) override;
	virtual bool SetLooping(bool Looping) override;
	virtual bool SetRate(float Rate) override;
	virtual bool SetNativeVolume(float Volume) override;

protected:

	//~ IMediaTracks interface

	virtual bool GetAudioTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaAudioTrackFormat& OutFormat) const override;
	virtual int32 GetNumTracks(EMediaTrackType TrackType) const override;
	virtual int32 GetNumTrackFormats(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetSelectedTrack(EMediaTrackType TrackType) const override;
	virtual FText GetTrackDisplayName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual int32 GetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackLanguage(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual FString GetTrackName(EMediaTrackType TrackType, int32 TrackIndex) const override;
	virtual bool GetVideoTrackFormat(int32 TrackIndex, int32 FormatIndex, FMediaVideoTrackFormat& OutFormat) const override;
	virtual bool SelectTrack(EMediaTrackType TrackType, int32 TrackIndex) override;
	virtual bool SetTrackFormat(EMediaTrackType TrackType, int32 TrackIndex, int32 FormatIndex) override;

private:

	/** Callback for when the application resumed in the foreground. */
	void HandleApplicationHasEnteredForeground();

	/** Callback for when the application is being paused in the background. */
	void HandleApplicationWillEnterBackground();

private:

	/** Audio track descriptors. */
	TArray<FJavaAndroidMediaPlayer::FAudioTrack> AudioTracks;

	/** Caption track descriptors. */
	TArray<FJavaAndroidMediaPlayer::FCaptionTrack> CaptionTracks;

	/** Video track descriptors. */
	TArray<FJavaAndroidMediaPlayer::FVideoTrack> VideoTracks;

private:

	/** Current player state. */
	EMediaState CurrentState;

	/** Current state of looping. */
	bool bLooping;

	/** The media event handler. */
	IMediaEventSink& EventSink;

	/** Media information string. */
	FString Info;

	/** The Java side media interface. */
	TSharedPtr<FJavaAndroidMediaPlayer, ESPMode::ThreadSafe> JavaMediaPlayer;

	/** Currently opened media. */
	FString MediaUrl;

	/** Media player Guid */
	FGuid PlayerGuid;

	/** Foreground/background delegate for pause. */
	FDelegateHandle PauseHandle;

	/** Foreground/background delegate for resume. */
	FDelegateHandle ResumeHandle;

	/** The media sample queue. */
	TSharedPtr<FMediaSamples, ESPMode::ThreadSafe> Samples;

	/** Index of the selected audio track. */
	int32 SelectedAudioTrack;

	/** Index of the selected audio track. */
	int32 SelectedCaptionTrack;

	/** Index of the selected video track. */
	int32 SelectedVideoTrack;

	/** Video sample object pool. */
	FAndroidMediaTextureSamplePool* VideoSamplePool;

	/** Device enters background. */
	bool bEnterBackground;
};
