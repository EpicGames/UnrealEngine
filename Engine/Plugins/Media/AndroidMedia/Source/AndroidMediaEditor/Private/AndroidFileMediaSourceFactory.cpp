// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFileMediaSourceFactory.h"
#include "FileMediaSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AndroidFileMediaSourceFactory)


/* UAndroidFileMediaSourceFactory structors
 *****************************************************************************/

UAndroidFileMediaSourceFactory::UAndroidFileMediaSourceFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	Formats.Add(TEXT("3gpp;3GPP Multimedia File"));
	Formats.Add(TEXT("aac;MPEG-2 Advanced Audio Coding File"));
	Formats.Add(TEXT("mp4;MPEG-4 Movie"));
	Formats.Add(TEXT("webm;WEBM Movie"));

	SupportedClass = UFileMediaSource::StaticClass();
	bEditorImport = true;
}


/* UFactory overrides
 *****************************************************************************/

bool UAndroidFileMediaSourceFactory::FactoryCanImport(const FString& Filename)
{
	return true;
}


UObject* UAndroidFileMediaSourceFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	UFileMediaSource* MediaSource = NewObject<UFileMediaSource>(InParent, InClass, InName, Flags);
	MediaSource->SetFilePath(CurrentFilename);

	return MediaSource;
}

