// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"


/**
 * Implements the AndroidMediaEditor module.
 */
class FAndroidMediaEditorModule
	: public IModuleInterface
{
public:

	//~ IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FAndroidMediaEditorModule, AndroidMediaEditor);
