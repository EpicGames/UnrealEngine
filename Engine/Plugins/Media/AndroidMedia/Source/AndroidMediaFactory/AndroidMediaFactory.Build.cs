// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AndroidMediaFactory : ModuleRules
	{
		public AndroidMediaFactory(ReadOnlyTargetRules Target) : base(Target)
		{
			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
					"Media",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"MediaAssets",
				});

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AndroidMedia",
					"Media",
				});

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
				});

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				DynamicallyLoadedModuleNames.Add("AndroidMedia");
			}
		}
	}
}
