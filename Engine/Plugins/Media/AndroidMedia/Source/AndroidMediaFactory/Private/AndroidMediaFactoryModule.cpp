// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidMediaFactoryPrivate.h"

#include "Internationalization/Internationalization.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "Modules/ModuleManager.h"
#include "IMediaModule.h"
#include "IMediaPlayerFactory.h"

#include "IAndroidMediaModule.h"


DEFINE_LOG_CATEGORY(LogAndroidMediaFactory);

#define LOCTEXT_NAMESPACE "FAndroidMediaFactoryModule"


/**
 * Implements the AndroidMediaFactory module.
 */
class FAndroidMediaFactoryModule
	: public IMediaPlayerFactory
	, public IModuleInterface
{
public:

	//~ IMediaPlayerFactory interface

	virtual bool CanPlayUrl(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		return GetPlayabilityConfidenceScore(Url, Options, OutWarnings, OutErrors) > 0 ? true : false;
	}

	virtual int32 GetPlayabilityConfidenceScore(const FString& Url, const IMediaOptions* Options, TArray<FText>* OutWarnings, TArray<FText>* OutErrors) const override
	{
		FString Scheme;
		FString Location;

		// check scheme
		if (!Url.Split(TEXT("://"), &Scheme, &Location, ESearchCase::CaseSensitive))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(LOCTEXT("NoSchemeFound", "No URI scheme found"));
			}

			return 0;
		}

		if (!SupportedUriSchemes.Contains(Scheme))
		{
			if (OutErrors != nullptr)
			{
				OutErrors->Add(FText::Format(LOCTEXT("SchemeNotSupported", "The URI scheme '{0}' is not supported"), FText::FromString(Scheme)));
			}

			return 0;
		}

		// check file extension
		if (Scheme == TEXT("file"))
		{
			const FString Extension = FPaths::GetExtension(Location, false);

			if (!SupportedFileExtensions.Contains(Extension))
			{
				if (OutErrors != nullptr)
				{
					OutErrors->Add(FText::Format(LOCTEXT("ExtensionNotSupported", "The file extension '{0}' is not supported"), FText::FromString(Extension)));
				}

				return 0;
			}
		}

		return 80;
	}

	virtual TSharedPtr<IMediaPlayer, ESPMode::ThreadSafe> CreatePlayer(IMediaEventSink& EventSink) override
	{
		auto AndroidMediaModule = FModuleManager::LoadModulePtr<IAndroidMediaModule>("AndroidMedia");
		return (AndroidMediaModule != nullptr) ? AndroidMediaModule->CreatePlayer(EventSink) : nullptr;
	}

	virtual FText GetDisplayName() const override
	{
		return LOCTEXT("MediaPlayerDisplayName", "Android Media");
	}

	virtual FName GetPlayerName() const override
	{
		static FName PlayerName(TEXT("AndroidMedia"));
		return PlayerName;
	}

	virtual FGuid GetPlayerPluginGUID() const override
	{
		static FGuid PlayerPluginGUID(0x894a9ab3, 0xb44d4373, 0x87a7dd0c, 0x9cbd9613);
		return PlayerPluginGUID;
	}

	virtual const TArray<FString>& GetSupportedPlatforms() const override
	{
		return SupportedPlatforms;
	}

	virtual bool SupportsFeature(EMediaFeature Feature) const override
	{
		return ((Feature == EMediaFeature::AudioTracks) ||
				(Feature == EMediaFeature::VideoSamples) ||
				(Feature == EMediaFeature::VideoTracks));
	}

public:

	//~ IModuleInterface interface

	virtual void StartupModule() override
	{
		// supported file extensions
		SupportedFileExtensions.Add(TEXT("3gpp"));
		SupportedFileExtensions.Add(TEXT("aac"));
		SupportedFileExtensions.Add(TEXT("mp4"));
		SupportedFileExtensions.Add(TEXT("m3u8"));
		SupportedFileExtensions.Add(TEXT("webm"));

		// supported platforms
		SupportedPlatforms.Add(TEXT("Android"));

		// supported schemes
		SupportedUriSchemes.Add(TEXT("file"));
		SupportedUriSchemes.Add(TEXT("http"));
		SupportedUriSchemes.Add(TEXT("httpd"));
		SupportedUriSchemes.Add(TEXT("https"));
		SupportedUriSchemes.Add(TEXT("mms"));
		SupportedUriSchemes.Add(TEXT("rtsp"));
		SupportedUriSchemes.Add(TEXT("rtspt"));
		SupportedUriSchemes.Add(TEXT("rtspu"));

		// register media player info
		auto MediaModule = FModuleManager::LoadModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->RegisterPlayerFactory(*this);
		}
	}

	virtual void ShutdownModule() override
	{
		// unregister player factory
		auto MediaModule = FModuleManager::GetModulePtr<IMediaModule>("Media");

		if (MediaModule != nullptr)
		{
			MediaModule->UnregisterPlayerFactory(*this);
		}
	}

private:

	/** List of supported media file types. */
	TArray<FString> SupportedFileExtensions;

	/** List of platforms that the media player support. */
	TArray<FString> SupportedPlatforms;

	/** List of supported URI schemes. */
	TArray<FString> SupportedUriSchemes;
};


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAndroidMediaFactoryModule, AndroidMediaFactory);
